// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TcpMessagingSettings.generated.h"


UCLASS(config=Engine)
class UTcpMessagingSettings
	: public UObject
{
	GENERATED_BODY()

public:

	/** Returns true if our transport is enabled */
	bool IsTransportEnabled() const;

	/** Gets the listen endpoint */
	FString GetListenEndpoint() const;

	/** Returns array of connect endpoints */
	void GetConnectToEndpoints(TArray<FString>& Endpoints) const;

	/** Returns reconnection delay */
	int32 GetConnectionRetryDelay() const;

	/** Returns reconnection period */
	int32 GetConnectionRetryPeriod() const;

	/** Returns whether to stop the transport service when the application deactivates */
	bool ShouldStopServiceWhenAppDeactivates() const;

private:

	/** Whether the TCP transport channel is enabled */
	UPROPERTY(config, EditAnywhere, Category = Transport)
	bool EnableTransport;

	/**
	 * The IP endpoint to listen for incoming connections.
	 *
	 * The format is IP_ADDRESS:PORT_NUMBER or blank to disable listening.
	 */
	UPROPERTY(config, EditAnywhere, Category=Transport)
	FString ListenEndpoint;

	/**
	 * The IP endpoints to try to establish outgoing connection to.
	 *
	 * Use this setting to connect to a remote peer.
	 * The format is IP_ADDRESS:PORT_NUMBER.
	 */
	UPROPERTY(config, EditAnywhere, Category=Transport)
	TArray<FString> ConnectToEndpoints;

	/**
	 * Delay time between attempts to re-establish outgoing connections that become disconnected or fail to connect
	 * 0 disables reconnection
	 */
	UPROPERTY(config, EditAnywhere, Category = Transport)
	int32 ConnectionRetryDelay;

	/**
	 * Period time during which attempts to re-establish outgoing connections that become disconnected or fail to connect
	 * 0 means it will be retried only once
	 */
	UPROPERTY(config, EditAnywhere, Category = Transport)
	int32 ConnectionRetryPeriod;

	/** Whether to stop the transport service when the application deactivates, and restart it when the application is reactivated */
	UPROPERTY(config)
	bool bStopServiceWhenAppDeactivates = true;
};
