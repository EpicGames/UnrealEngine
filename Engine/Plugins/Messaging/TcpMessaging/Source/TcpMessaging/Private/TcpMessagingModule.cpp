// Copyright Epic Games, Inc. All Rights Reserved.

#include "TcpMessagingPrivate.h"

#include "MessageBridgeBuilder.h"
#include "Misc/CoreDelegates.h"
#include "Misc/App.h"
#include "Settings/TcpMessagingSettings.h"

#if WITH_EDITOR
	#include "ISettingsModule.h"
	#include "ISettingsSection.h"
#endif

#include "ITcpMessagingModule.h"
#include "Transport/TcpMessageTransport.h"


DEFINE_LOG_CATEGORY(LogTcpMessaging);

#define LOCTEXT_NAMESPACE "FTcpMessagingModule"


/**
 * Implements the TcpMessagingModule module.
 */
class FTcpMessagingModule
	: public FSelfRegisteringExec
	, public ITcpMessagingModule
{
protected:

	// FSelfRegisteringExec interface

	virtual bool Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		if (!FParse::Command(&Cmd, TEXT("TCPMESSAGING")))
		{
			return false;
		}

		if (FParse::Command(&Cmd, TEXT("STATUS")))
		{
			// general information
			Ar.Logf(TEXT("Protocol Version: %d"), (int32)ETcpMessagingVersion::LatestVersion);

			// bridge status
			if (MessageBridge.IsValid())
			{
				if (MessageBridge->IsEnabled())
				{
					Ar.Log(TEXT("Message Bridge: Initialized and enabled"));
				}
				else
				{
					Ar.Log(TEXT("Message Bridge: Initialized, but disabled"));
				}
			}
			else
			{
				Ar.Log(TEXT("Message Bridge: Not initialized."));
			}
		}
		else if (FParse::Command(&Cmd, TEXT("RESTART")))
		{
			RestartServices();
		}
		else if (FParse::Command(&Cmd, TEXT("SHUTDOWN")))
		{
			ShutdownBridge();
		}
		else
		{
			// show usage
			Ar.Log(TEXT("Usage: TCPMESSAGING <Command>"));
			Ar.Log(TEXT(""));
			Ar.Log(TEXT("Command"));
			Ar.Log(TEXT("    RESTART = Restarts the message bridge, if enabled"));
			Ar.Log(TEXT("    SHUTDOWN = Shut down the message bridge, if running"));
			Ar.Log(TEXT("    STATUS = Displays the status of the TCP message transport"));
		}

		return true;
	}

public:

	// IModuleInterface interface

	virtual void StartupModule() override
	{
		if (!SupportsNetworkedTransport())
		{
			return;
		}

		// load dependencies
		if (FModuleManager::Get().LoadModule(TEXT("Networking")) == nullptr)
		{
			UE_LOGF(LogTcpMessaging, Error, "The required module 'Networking' failed to load. Plug-in 'Tcp Messaging' cannot be used.");

			return;
		}

#if WITH_EDITOR
		// register settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			ISettingsSectionPtr SettingsSection = SettingsModule->RegisterSettings("Project", "Plugins", "TcpMessaging",
				LOCTEXT("TcpMessagingSettingsName", "TCP Messaging"),
				LOCTEXT("TcpMessagingSettingsDescription", "Configure the TCP Messaging plug-in."),
				GetMutableDefault<UTcpMessagingSettings>()
				);

			if (SettingsSection.IsValid())
			{
				SettingsSection->OnModified().BindRaw(this, &FTcpMessagingModule::HandleSettingsSaved);
			}
		}
#endif // WITH_EDITOR

		// register application events
		const UTcpMessagingSettings* Settings = GetDefault<UTcpMessagingSettings>();
		if (Settings->ShouldStopServiceWhenAppDeactivates())
		{
			FCoreDelegates::ApplicationHasReactivatedDelegate.AddRaw(this, &FTcpMessagingModule::HandleApplicationHasReactivated);
			FCoreDelegates::ApplicationWillDeactivateDelegate.AddRaw(this, &FTcpMessagingModule::HandleApplicationWillDeactivate);
		}

		RestartServices();
	}

	virtual void ShutdownModule() override
	{
		// unregister application events
		FCoreDelegates::ApplicationHasReactivatedDelegate.RemoveAll(this);
		FCoreDelegates::ApplicationWillDeactivateDelegate.RemoveAll(this);

#if WITH_EDITOR
		// unregister settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Plugins", "TcpMessaging");
		}
#endif

		// shut down services
		ShutdownBridge();
	}

	virtual bool SupportsDynamicReloading() override
	{
		return true;
	}

	virtual void AddOutgoingConnection(const FString& EndpointString) override
	{
		FIPv4Endpoint Endpoint;
		if (FIPv4Endpoint::Parse(EndpointString, Endpoint))
		{
			auto Transport = MessageTransportPtr.Pin();
			if (Transport.IsValid())
			{
				Transport->AddOutgoingConnection(Endpoint);
			}
		}
	}

	virtual void RemoveOutgoingConnection(const FString& EndpointString) override
	{
		FIPv4Endpoint Endpoint;
		if (FIPv4Endpoint::Parse(EndpointString, Endpoint))
		{
			auto Transport = MessageTransportPtr.Pin();
			if (Transport.IsValid())
			{
				Transport->RemoveOutgoingConnection(Endpoint);
			}
		}
	}

protected:

	/** Initializes the message bridge with the current settings. */
	void InitializeBridge()
	{
		ShutdownBridge();

		const UTcpMessagingSettings* Settings = GetDefault<UTcpMessagingSettings>();
		bool ResaveSettings = false;

		FIPv4Endpoint ListenEndpoint;
		TArray<FIPv4Endpoint> ConnectToEndpoints;

		FString ListenEndpointString = Settings->GetListenEndpoint();

		if (!FIPv4Endpoint::Parse(ListenEndpointString, ListenEndpoint))
		{
			if (!ListenEndpointString.IsEmpty())
			{
				UE_LOGF(LogTcpMessaging, Warning, "Invalid setting for ListenEndpoint '%ls', listening disabled", *ListenEndpointString);
			}

			ListenEndpoint = FIPv4Endpoint::Any;
		}
		
		TArray<FString> ConnectToEndpointStrings;
		Settings->GetConnectToEndpoints(ConnectToEndpointStrings);

		for (FString& ConnectToEndpointString : ConnectToEndpointStrings)
		{
			FIPv4Endpoint ConnectToEndpoint;
			if (FIPv4Endpoint::Parse(ConnectToEndpointString, ConnectToEndpoint) || FIPv4Endpoint::FromHostAndPort(ConnectToEndpointString, ConnectToEndpoint))
			{
				ConnectToEndpoints.Add(ConnectToEndpoint);
			}
			else
			{
				UE_LOGF(LogTcpMessaging, Warning, "Invalid entry for ConnectToEndpoint '%ls', ignoring", *ConnectToEndpointString);
			}
		}

		FString Status(TEXT("Initializing TcpMessaging bridge"));
		if (ConnectToEndpoints.Num())
		{
			Status += FString::Printf(TEXT(" for %d outgoing connections"), ConnectToEndpoints.Num());
		}
		if (ListenEndpoint != FIPv4Endpoint::Any)
		{
			Status += TEXT(", listening on ");
			Status += ListenEndpoint.ToText().ToString();
		}

		UE_LOGF(LogTcpMessaging, Log, "%ls", *Status);

		TSharedRef<FTcpMessageTransport, ESPMode::ThreadSafe> Transport = MakeShareable(new FTcpMessageTransport(ListenEndpoint, ConnectToEndpoints, Settings->GetConnectionRetryDelay(), Settings->GetConnectionRetryPeriod()));
		
		// Safe weak pointer for adding/removing connections
		MessageTransportPtr = Transport;

		MessageBridge = FMessageBridgeBuilder().UsingTransport(Transport);
	}

	/** Restarts the bridge service. */
	void RestartServices()
	{
		const UTcpMessagingSettings* Settings = GetDefault<UTcpMessagingSettings>();

		if (Settings->IsTransportEnabled())
		{
			InitializeBridge();
		}
		else
		{
			ShutdownBridge();
		}
	}

	/**
	 * Checks whether networked message transport is supported.
	 *
	 * @return true if networked transport is supported, false otherwise.
	 */
	bool SupportsNetworkedTransport() const
	{
#if UE_BUILD_SHIPPING && !(defined(ALLOW_TCP_MESSAGING_SHIPPING) && ALLOW_TCP_MESSAGING_SHIPPING)
		return false;
#else
		// disallow unsupported platforms
		if (!FPlatformMisc::SupportsMessaging())
		{
			return false;
		}

		// single thread support not implemented yet
		if (!FPlatformProcess::SupportsMultithreading())
		{
			return false;
		}

		// always allow in standalone Slate applications
		if (!FApp::IsGame() && !IsRunningCommandlet())
		{
			return true;
		}

		// otherwise only allow if explicitly desired
		return FParse::Param(FCommandLine::Get(), TEXT("Messaging"));
#endif
	}

	/** Shuts down the message bridge. */
	void ShutdownBridge()
	{
		if (MessageBridge.IsValid())
		{
			MessageBridge->Disable();
			FPlatformProcess::Sleep(0.1f);
			MessageBridge.Reset();
		}
	}

private:

	/** Callback for when an has been reactivated (i.e. return from sleep on iOS). */
	void HandleApplicationHasReactivated()
	{
		RestartServices();
	}

	/** Callback for when the application will be deactivated (i.e. sleep on iOS).*/
	void HandleApplicationWillDeactivate()
	{
		ShutdownBridge();
	}

	/** Callback for when the settings were saved. */
	bool HandleSettingsSaved()
	{
		RestartServices();

		return true;
	}

private:

	/** Holds the message bridge if present. */
	TSharedPtr<IMessageBridge, ESPMode::ThreadSafe> MessageBridge;
	
	/** Message transport pointer, if valid */
	TWeakPtr<FTcpMessageTransport, ESPMode::ThreadSafe> MessageTransportPtr;
};

IMPLEMENT_MODULE(FTcpMessagingModule, TcpMessaging);

bool UTcpMessagingSettings::IsTransportEnabled() const
{
	if (EnableTransport)
	{
		return true;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("TcpMessagingListen=")) || FParse::Param(FCommandLine::Get(), TEXT("TcpMessagingConnect=")))
	{
		return true;
	}

	return false;
}

FString UTcpMessagingSettings::GetListenEndpoint() const
{
	FString HostString = ListenEndpoint;
	
	// Read command line override
	FParse::Value(FCommandLine::Get(), TEXT("TcpMessagingListen="), HostString);

	return HostString;
}

void UTcpMessagingSettings::GetConnectToEndpoints(TArray<FString>& Endpoints) const
{
	for (const FString& ConnectEndpoint : ConnectToEndpoints)
	{
		Endpoints.Add(ConnectEndpoint);
	}

	FString ConnectString;
	if (FParse::Value(FCommandLine::Get(), TEXT("TcpMessagingConnect="), ConnectString, /* bShouldStopOnSeparator */ false))
	{
		ConnectString.ParseIntoArray(Endpoints, TEXT(","));
	}
}

int32 UTcpMessagingSettings::GetConnectionRetryDelay() const
{
	return ConnectionRetryDelay;
}

int32 UTcpMessagingSettings::GetConnectionRetryPeriod() const
{
	return ConnectionRetryPeriod;
}

bool UTcpMessagingSettings::ShouldStopServiceWhenAppDeactivates() const
{
	return bStopServiceWhenAppDeactivates;
}

#undef LOCTEXT_NAMESPACE
