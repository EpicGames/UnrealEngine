// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

/* Private constants
 *****************************************************************************/

/** Declares a log category for this module. */
DECLARE_LOG_CATEGORY_EXTERN(LogTcpMessaging, Log, All);

/** Defines the maximum number of annotations a message can have. */
#define TCP_MESSAGING_MAX_ANNOTATIONS 128

/** Defines the maximum number of recipients a message can have. */
#define TCP_MESSAGING_MAX_RECIPIENTS 1024

/** Defines the desired size of socket send buffers (in bytes). */
#define TCP_MESSAGING_SEND_BUFFER_SIZE 2 * 1024 * 1024

/** Defines the desired size of socket receive buffers (in bytes). */
#define TCP_MESSAGING_RECEIVE_BUFFER_SIZE 2 * 1024 * 1024

/** Defines a magic number for the the TCP message transport. */
#define TCP_MESSAGING_TRANSPORT_PROTOCOL_MAGIC 0x45504943

/** Defines the protocol version of the TCP message transport. */
namespace ETcpMessagingVersion
{
	enum Type
	{
		Initial,
		ChangedMessageLengthToInt32,
		ChangedMessageContext,

		// -----<new versions can be added before this line>-------------------------------------------------
		// - this needs to be the last line (see note below)
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1,
		// bump this when break need to break compatibility.
		OldestSupportedVersion = ChangedMessageLengthToInt32
	};
}
