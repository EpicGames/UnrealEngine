// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transport/TcpDeserializedMessage.h"
#include "Async/TaskGraphInterfaces.h"
#include "TcpMessagingPrivate.h"
#include "Backends/JsonStructDeserializerBackend.h"
#include "Serialization/ArrayReader.h"
#include "StructDeserializer.h"


/* FTcpDeserializedMessage structors
*****************************************************************************/

FTcpDeserializedMessage::FTcpDeserializedMessage(const TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe>& InAttachment)
	: Attachment(InAttachment)
	, MessageData(nullptr)
{ }


FTcpDeserializedMessage::~FTcpDeserializedMessage()
{
	if (MessageData != nullptr)
	{
		if (UScriptStruct* TypeInfoPtr = TypeInfo.Get())
		{
			TypeInfoPtr->DestroyStruct(MessageData);
		}

		FMemory::Free(MessageData);
		MessageData = nullptr;
	}
}


/* FTcpDeserializedMessage interface
 *****************************************************************************/

bool FTcpDeserializedMessage::Deserialize(const TSharedPtr<FArrayReader, ESPMode::ThreadSafe>& Message)
{
	FArrayReader& MessageReader = Message.ToSharedRef().Get();

	// Note that some complex values are deserialized manually here, so that we
	// can sanity check their values. @see FTcpSerializeMessageTask::DoTask()
	MessageReader.ArMaxSerializeSize = NAME_SIZE;

	UScriptStruct* TypeInfoPtr = nullptr;

	// message type info
	{
		FTopLevelAssetPath MessageType;
		MessageReader << MessageType;

		while (IsGarbageCollectingAndLockingUObjectHashTables())
		{
			// Wait for garbage collector
			FPlatformProcess::Sleep(0.1f);
		}
		TypeInfoPtr = FindObjectSafe<UScriptStruct>(MessageType);

		TypeInfo = TypeInfoPtr;
		if (!TypeInfo.IsValid(false, true))
		{
			UE_LOGF(LogTcpMessaging, Verbose, "Lookup for Message Type '%ls' failed during Deserialization.", *MessageType.ToString());
			return false;
		}
	}
	check(TypeInfoPtr);

	// sender address
	{
		MessageReader << Sender;
	}

	// recipient addresses
	{
		int32 NumRecipients = 0;
		MessageReader << NumRecipients;

		if ((NumRecipients < 0) || (NumRecipients > TCP_MESSAGING_MAX_RECIPIENTS))
		{
			return false;
		}

		Recipients.Empty(NumRecipients);

		while (0 < NumRecipients--)
		{
			MessageReader << *::new(Recipients) FMessageAddress;
		}
	}

	// message scope
	{
		MessageReader << Scope;

		if (Scope > EMessageScope::All)
		{
			return false;
		}
	}

	// time sent & expiration
	{
		MessageReader << TimeSent;
		MessageReader << Expiration;
	}

	// annotations
	{
		int32 NumAnnotations = 0;
		MessageReader << NumAnnotations;

		if (NumAnnotations > TCP_MESSAGING_MAX_ANNOTATIONS)
		{
			return false;
		}

		while (0 < NumAnnotations--)
		{
			FName Key;
			FString Value;

			MessageReader << Key;
			MessageReader << Value;

			Annotations.Add(Key, Value);
		}
	}

	// create message body
	MessageData = FMemory::Malloc(TypeInfoPtr->GetStructureSize());
	TypeInfoPtr->InitializeStruct(MessageData);

	// deserialize message body
	FJsonStructDeserializerBackend Backend(MessageReader);

	return FStructDeserializer::Deserialize(MessageData, *TypeInfoPtr, Backend);
}


/* IMessageContext interface
 *****************************************************************************/

const TMap<FName, FString>& FTcpDeserializedMessage::GetAnnotations() const
{
	return Annotations;
}


TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe> FTcpDeserializedMessage::GetAttachment() const
{
	return Attachment;
}


const FDateTime& FTcpDeserializedMessage::GetExpiration() const
{
	return Expiration;
}


const void* FTcpDeserializedMessage::GetMessage() const
{
	return MessageData;
}


const TWeakObjectPtr<UScriptStruct>& FTcpDeserializedMessage::GetMessageTypeInfo() const
{
	return TypeInfo;
}


TSharedPtr<IMessageContext, ESPMode::ThreadSafe> FTcpDeserializedMessage::GetOriginalContext() const
{
	return nullptr;
}


const TArray<FMessageAddress>& FTcpDeserializedMessage::GetRecipients() const
{
	return Recipients;
}


EMessageScope FTcpDeserializedMessage::GetScope() const
{
	return Scope;
}

EMessageFlags FTcpDeserializedMessage::GetFlags() const
{
	return EMessageFlags::None;
}


const FMessageAddress& FTcpDeserializedMessage::GetSender() const
{
	return Sender;
}


const FMessageAddress& FTcpDeserializedMessage::GetForwarder() const
{
	return Sender;
}


ENamedThreads::Type FTcpDeserializedMessage::GetSenderThread() const
{
	return ENamedThreads::AnyThread;
}


const FDateTime& FTcpDeserializedMessage::GetTimeForwarded() const
{
	return TimeSent;
}


const FDateTime& FTcpDeserializedMessage::GetTimeSent() const
{
	return TimeSent;
}
