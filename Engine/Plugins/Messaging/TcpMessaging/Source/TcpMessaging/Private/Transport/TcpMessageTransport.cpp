// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transport/TcpMessageTransport.h"

#include "Common/TcpListener.h"
#include "IMessageTransportHandler.h"
#include "TcpMessagingPrivate.h"

#include "TcpSerializedMessage.h"
#include "Transport/TcpDeserializedMessage.h"
#include "Transport/TcpMessageTransportConnection.h"
#include "Transport/TcpSerializeMessageTask.h"


/* FTcpMessageTransport structors
 *****************************************************************************/

FTcpMessageTransport::FTcpMessageTransport(const FIPv4Endpoint& InListenEndpoint, const TArray<FIPv4Endpoint>& InConnectToEndpoints, int32 InConnectionRetryDelay, int32 InConnectionRetryPeriod)
	: ListenEndpoint(InListenEndpoint)
	, ConnectToEndpoints(InConnectToEndpoints)
	, ConnectionRetryDelay(InConnectionRetryDelay)
	, ConnectionRetryPeriod(InConnectionRetryPeriod)
	, bStopping(false)
	, SocketSubsystem(ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM))
	, Listener(nullptr)
	, TransportHandler(nullptr)
{
	Thread = FRunnableThread::Create(this, TEXT("FTcpMessageTransport"), 128 * 1024, TPri_Normal);
}


FTcpMessageTransport::~FTcpMessageTransport()
{
	if (Thread != nullptr)
	{
		Thread->Kill(true);
		delete Thread;
	}

	StopTransport();
}


/* IMessageTransport interface
 *****************************************************************************/

bool FTcpMessageTransport::StartTransport(IMessageTransportHandler& Handler)
{
	TransportHandler = &Handler;

	if (ListenEndpoint != FIPv4Endpoint::Any)
	{
		Listener = new FTcpListener(ListenEndpoint);
		Listener->OnConnectionAccepted().BindRaw(this, &FTcpMessageTransport::HandleListenerConnectionAccepted);
	}

	// outgoing connections
	for (auto& ConnectToEndPoint : ConnectToEndpoints)
	{
		AddOutgoingConnection(ConnectToEndPoint);
	}

	return true;
}

void FTcpMessageTransport::AddOutgoingConnection(const FIPv4Endpoint& Endpoint)
{
	FSocket* Socket = FTcpSocketBuilder(TEXT("FTcpMessageTransport.RemoteConnection"));
	
	if (Socket == nullptr)
	{
		UE_LOGF(LogTcpMessaging, Error, "Failed to create outgoing socket for %ls", *Endpoint.ToString());
		return;
	}

	if (!Socket->Connect(*Endpoint.ToInternetAddr()))
	{
		UE_LOGF(LogTcpMessaging, Error, "Connect failed on outgoing socket for %ls", *Endpoint.ToString());
		ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->DestroySocket(Socket);
	}
	else
	{
		PendingConnections.Enqueue(MakeShareable(new FTcpMessageTransportConnection(Socket, Endpoint, ConnectionRetryDelay, ConnectionRetryPeriod)));
	}
}

void FTcpMessageTransport::RemoveOutgoingConnection(const FIPv4Endpoint& Endpoint)
{
	ConnectionEndpointsToRemove.Enqueue(Endpoint);
}

void FTcpMessageTransport::StopTransport()
{
	bStopping = true;

	if (Listener)
	{
		delete Listener;
		Listener = nullptr;
	}

	for (auto& Connection : Connections)
	{
		Connection->Close();
	}

	Connections.Empty();
	PendingConnections.Empty();
	ConnectionEndpointsToRemove.Empty();

	TransportHandler = nullptr;
}


bool FTcpMessageTransport::TransportMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const TArray<FGuid>& Recipients)
{
	if (Context->GetRecipients().Num() > TCP_MESSAGING_MAX_RECIPIENTS)
	{
		return false;
	}

	// Handle any queued changes to the NodeConnectionMap
	FNodeConnectionMapUpdate UpdateInfo;
	while (NodeConnectionMapUpdates.Dequeue(UpdateInfo))
	{
		check(UpdateInfo.NodeId.IsValid());
		if (UpdateInfo.bNewNode)
		{
			TSharedPtr<FTcpMessageTransportConnection> ConnectionPtr = UpdateInfo.Connection.Pin();
			if (ConnectionPtr.IsValid())
			{
				NodeConnectionMap.Add(UpdateInfo.NodeId, ConnectionPtr);
			}
		}
		else
		{
			NodeConnectionMap.Remove(UpdateInfo.NodeId);
		}
	}

	// Work out which connections we need to send this message to.
	TArray<TSharedPtr<FTcpMessageTransportConnection>> RecipientConnections;

	if (Recipients.Num() == 0)
	{
		// broadcast the message to all valid connections
		RecipientConnections = Connections.FilterByPredicate([&](const TSharedPtr<FTcpMessageTransportConnection>& Connection) -> bool
		{
			return Connection->GetConnectionState() == FTcpMessageTransportConnection::STATE_Connected;
		});
	}
	else
	{
		// Find connections for each recipient.  We do not transport unicast messages for unknown nodes.
		for (auto& Recipient : Recipients)
		{
			TSharedPtr<FTcpMessageTransportConnection>* RecipientConnection = NodeConnectionMap.Find(Recipient);
			if (RecipientConnection && (*RecipientConnection)->GetConnectionState() == FTcpMessageTransportConnection::STATE_Connected)
			{
				RecipientConnections.AddUnique(*RecipientConnection);
			}
		}
	}

	if (RecipientConnections.Num() == 0)
	{
		UE_LOGF(LogTcpMessaging, Verbose, "No recipients for message %ls from %ls", *Context->GetMessageTypePathName().ToString(), *Context->GetSender().ToString());
		return false;
	}

	UE_LOGF(LogTcpMessaging, Verbose, "Transporting message '%ls' to %d connections", *Context->GetMessageTypePathName().ToString(), RecipientConnections.Num());

	FTcpSerializedMessageRef SerializedMessage = MakeShareable(new FTcpSerializedMessage());

	TGraphTask<FTcpSerializeMessageTask>::CreateTask().ConstructAndDispatchWhenReady(Context, SerializedMessage, RecipientConnections);

	return true;
}


/* FRunnable interface
*****************************************************************************/

void FTcpMessageTransport::Exit()
{
	// do nothing
}


bool FTcpMessageTransport::Init()
{
	return true;
}


uint32 FTcpMessageTransport::Run()
{
	while (!bStopping)
	{
		// new connections
		{
			TSharedPtr<FTcpMessageTransportConnection> Connection;
			while (PendingConnections.Dequeue(Connection))
			{
				Connection->OnTcpMessageTransportConnectionStateChanged().BindRaw(this, &FTcpMessageTransport::HandleConnectionStateChanged, Connection);
				Connection->Start();
				Connections.Add(Connection);
			}
		}

		// connections to remove
		{
			FIPv4Endpoint Endpoint;
			while (ConnectionEndpointsToRemove.Dequeue(Endpoint))
			{
				for (int32 Index = 0; Index < Connections.Num(); Index++)
				{
					auto& Connection = Connections[Index];

					if (Connection->GetRemoteEndpoint() == Endpoint)
					{
						Connection->Close();
						break;
					}
				}	
			}
		}

		int32 ActiveConnections = 0;
		for (int32 Index = 0;Index < Connections.Num(); Index++)
		{
			auto& Connection = Connections[Index];

			// handle disconnected by remote
			switch (Connection->GetConnectionState())
			{
			case FTcpMessageTransportConnection::STATE_Connected:
				ActiveConnections++;
				break;

			case FTcpMessageTransportConnection::STATE_Disconnected:
				Connections.RemoveAtSwap(Index);
				Index--;
				break;

			default:
				break;
			}
		}

		// incoming messages
		{
			for (auto& Connection : Connections)
			{
				TSharedPtr<FTcpDeserializedMessage, ESPMode::ThreadSafe> Message;
				FGuid SenderNodeId;

				while (Connection->Receive(Message, SenderNodeId))
				{
					UE_LOGF(LogTcpMessaging, Verbose, "Received message '%ls'", *Message->GetMessageTypePathName().ToString());
					TransportHandler->ReceiveTransportMessage(Message.ToSharedRef(), SenderNodeId);
				}
			}
		}
				
		FPlatformProcess::Sleep(ActiveConnections > 0 ? 0.01f : 1.f);
	}

	return 0;
}


void FTcpMessageTransport::Stop()
{
	bStopping = true;
}


/* FTcpMessageTransport callbacks
*****************************************************************************/

bool FTcpMessageTransport::HandleListenerConnectionAccepted(FSocket* ClientSocket, const FIPv4Endpoint& ClientEndpoint)
{
	PendingConnections.Enqueue(MakeShareable(new FTcpMessageTransportConnection(ClientSocket, ClientEndpoint, 0)));
	
	return true;
}


void FTcpMessageTransport::HandleConnectionStateChanged(TSharedPtr<FTcpMessageTransportConnection> Connection)
{
	const FGuid NodeId = Connection->GetRemoteNodeId();
	const FIPv4Endpoint RemoteEndpoint = Connection->GetRemoteEndpoint();
	const FTcpMessageTransportConnection::EConnectionState State = Connection->GetConnectionState();

	if (State == FTcpMessageTransportConnection::STATE_Connected)
	{
		NodeConnectionMapUpdates.Enqueue(FNodeConnectionMapUpdate(true, NodeId, TWeakPtr<FTcpMessageTransportConnection>(Connection)));
		TransportHandler->DiscoverTransportNode(NodeId);

		UE_LOGF(LogTcpMessaging, Log, "Discovered node '%ls' on connection '%ls'...", *NodeId.ToString(), *RemoteEndpoint.ToString());
	}
	else if (NodeId.IsValid())
	{
		UE_LOGF(LogTcpMessaging, Log, "Lost node '%ls' on connection '%ls'...", *NodeId.ToString(), *RemoteEndpoint.ToString());

		NodeConnectionMapUpdates.Enqueue(FNodeConnectionMapUpdate(false, NodeId, TWeakPtr<FTcpMessageTransportConnection>(Connection)));
		TransportHandler->ForgetTransportNode(NodeId);
	}
}
