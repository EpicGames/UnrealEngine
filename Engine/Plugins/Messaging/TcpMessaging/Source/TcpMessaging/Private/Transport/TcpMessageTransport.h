// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "HAL/Runnable.h"
#include "Containers/Queue.h"
#include "IMessageTransport.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"

class FSocket;
class FTcpListener;
class FTcpMessageTransportConnection;


/** Entry specfying addition or removal to/from the NodeConnectionMap */
struct FNodeConnectionMapUpdate
{
	FNodeConnectionMapUpdate(bool bInNewNode, const FGuid& InNodeId, const TWeakPtr<FTcpMessageTransportConnection>& InConnection)
		: bNewNode(bInNewNode)
		, NodeId(InNodeId)
		, Connection(InConnection)
	{ }

	FNodeConnectionMapUpdate()
		: bNewNode(false)
	{ }

	bool bNewNode;
	FGuid NodeId;
	TWeakPtr<FTcpMessageTransportConnection> Connection;
};


/**
 * Implements a message transport technology using an Tcp network connection.
 *
 * On platforms that support multiple processes, the transport is using two sockets,
 * one for per-process unicast sending/receiving, and one for multicast receiving.
 * Console and mobile platforms use a single multicast socket for all sending and receiving.
 */
class FTcpMessageTransport
	: FRunnable
	, public IMessageTransport
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 */
	FTcpMessageTransport(const FIPv4Endpoint& InListenEndpoint, const TArray<FIPv4Endpoint>& InConnectToEndpoints, int32 InConnectionRetryDelay, int32 InConnectionRetryPeriod = 0);

	/** Virtual destructor. */
	virtual ~FTcpMessageTransport();

public:

	void AddOutgoingConnection(const FIPv4Endpoint& Endpoint);
	void RemoveOutgoingConnection(const FIPv4Endpoint& Endpoint);

public:

	//~ IMessageTransport interface

	virtual FName GetDebugName() const override
	{
		return "TcpMessageTransport";
	}

	virtual bool StartTransport(IMessageTransportHandler& Handler) override;
	virtual void StopTransport() override;
	virtual bool TransportMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const TArray<FGuid>& Recipients) override;

public:

	//~ FRunnable interface

	virtual void Exit() override;
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;

private:

	/** Callback for accepted connections to the local server. */
	bool HandleListenerConnectionAccepted(FSocket* ClientSocket, const FIPv4Endpoint& ClientEndpoint);

	/** Callback from connections for node discovery/loss */
	void HandleConnectionStateChanged(TSharedPtr<FTcpMessageTransportConnection> Connection);

private:

	/** Current settings */
	FIPv4Endpoint ListenEndpoint;
	TArray<FIPv4Endpoint> ConnectToEndpoints;
	int32 ConnectionRetryDelay;
	int32 ConnectionRetryPeriod;

	/** For the thread */
	bool bStopping;
	
	/** Holds a pointer to the socket sub-system. */
	ISocketSubsystem* SocketSubsystem;

	/** Holds the local listener for incoming tunnel connections. */
	FTcpListener* Listener;

	/** Current connections */
	TArray<TSharedPtr<FTcpMessageTransportConnection>> Connections;

	/** Map nodes to connections. We do not transport unicast messages for unknown nodes. */
	TMap<FGuid, TSharedPtr<FTcpMessageTransportConnection>> NodeConnectionMap;

	/** Holds a queue of changes to NodeConnectionMap. */
	TQueue<FNodeConnectionMapUpdate, EQueueMode::Mpsc> NodeConnectionMapUpdates;
	
	/** Holds a queue of pending connections. */
	TQueue<TSharedPtr<FTcpMessageTransportConnection>, EQueueMode::Mpsc> PendingConnections;

	/** Queue of enpoints describing connection we want to remove */
	TQueue<FIPv4Endpoint, EQueueMode::Mpsc> ConnectionEndpointsToRemove;

	/** Holds the thread object. */
	FRunnableThread* Thread;

	/** Message transport handler. */
	IMessageTransportHandler* TransportHandler;
};
