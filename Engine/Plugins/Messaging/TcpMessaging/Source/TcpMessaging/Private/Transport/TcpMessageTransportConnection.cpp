// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transport/TcpMessageTransportConnection.h"
#include "HAL/RunnableThread.h"
#include "Serialization/ArrayWriter.h"
#include "Common/TcpSocketBuilder.h"
#include "Serialization/ArrayReader.h"
#include "TcpMessagingPrivate.h"
#include "TcpSerializedMessage.h"
#include "Transport/TcpDeserializedMessage.h"

/** Header sent over the connection as soon as it's opened */
struct FTcpMessageHeader
{
	uint32 MagicNumber;
	uint32 Version;
	FGuid NodeId;

	FTcpMessageHeader()
	:	MagicNumber(0)
	,	Version(0)
	{}

	FTcpMessageHeader(const FGuid& InNodeId)
	:	MagicNumber(TCP_MESSAGING_TRANSPORT_PROTOCOL_MAGIC)
	,	Version(ETcpMessagingVersion::LatestVersion)
	,	NodeId(InNodeId)
	{}

	bool IsValid() const
	{
		return
			MagicNumber == TCP_MESSAGING_TRANSPORT_PROTOCOL_MAGIC &&
			Version >= ETcpMessagingVersion::OldestSupportedVersion &&
			NodeId.IsValid();
	}

	FGuid GetNodeId() const
	{
		return NodeId;
	}

	uint32 GetVersion() const
	{
		return Version;
	}

	// Serializer
	friend FArchive& operator<<(FArchive& Ar, FTcpMessageHeader& H)
	{
		return Ar << H.MagicNumber << H.Version << H.NodeId;
	}
};


/* FTcpMessageTransportConnection structors
 *****************************************************************************/

FTcpMessageTransportConnection::FTcpMessageTransportConnection(FSocket* InSocket, const FIPv4Endpoint& InRemoteEndpoint, int32 InConnectionRetryDelay, int32 InConnectionRetryPeriod)
	: ConnectionState(STATE_Connecting)
	, OpenedTime(FDateTime::UtcNow())
	, RemoteEndpoint(InRemoteEndpoint)
	, LocalNodeId(FGuid::NewGuid())
	, bSentHeader(false)
	, bReceivedHeader(false)
	, RemoteProtocolVersion(0)
	, Socket(InSocket)
	, Thread(nullptr)
	, TotalBytesReceived(0)
	, TotalBytesSent(0)
	, bRun(false)
	, ConnectionRetryDelay(InConnectionRetryDelay)
	, ConnectionRetryPeriod(InConnectionRetryPeriod)
	, RecvMessageDataRemaining(0)
{
	int32 NewSize = 0;
	Socket->SetReceiveBufferSize(TCP_MESSAGING_RECEIVE_BUFFER_SIZE, NewSize);
	Socket->SetSendBufferSize(TCP_MESSAGING_SEND_BUFFER_SIZE, NewSize);
}

FTcpMessageTransportConnection::~FTcpMessageTransportConnection()
{
	if (Thread != nullptr)
	{
		if(bRun)
		{
			bRun = false;
			Thread->WaitForCompletion();
		}
		delete Thread;
	}

	if (Socket)
	{
		ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->DestroySocket(Socket);
	}
}

/* FTcpMessageTransportConnection interface
 *****************************************************************************/

void FTcpMessageTransportConnection::Start()
{
	check(Thread == nullptr);
	bRun = true;
	Thread = FRunnableThread::Create(this, *FString::Printf(TEXT("FTcpMessageTransportConnection %s"), *RemoteEndpoint.ToString()), 128 * 1024, TPri_Normal);
}

bool FTcpMessageTransportConnection::Receive(TSharedPtr<FTcpDeserializedMessage, ESPMode::ThreadSafe>& OutMessage, FGuid& OutSenderNodeId)
{
	if(Inbox.Dequeue(OutMessage))
	{
		OutSenderNodeId = RemoteNodeId;
		return true;
	}
	return false;
}

bool FTcpMessageTransportConnection::Send(FTcpSerializedMessagePtr Message)
{
	FScopeLock SendLock(&SendCriticalSection);

	if (GetConnectionState() == STATE_Connected && bSentHeader)
	{
		int32 BytesSent = 0;
		const TArray<uint8>& Payload = Message->GetDataArray();

		// send the payload size
		FArrayWriter MessagesizeData = FArrayWriter(true);
		uint32 Messagesize = Payload.Num();
		MessagesizeData << Messagesize;

		if (!BlockingSend(MessagesizeData.GetData(), sizeof(uint32)))
		{
			UE_LOGF(LogTcpMessaging, Verbose, "Payload size write failed with code %d", (int32)ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLastErrorCode());
			return false;
		}

		TotalBytesSent += sizeof(uint32);

		// send the payload
		if (!ChunkedBlockingSend(Payload))
		{
			UE_LOGF(LogTcpMessaging, Verbose, "Payload write failed with code %d", (int32)ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLastErrorCode());
			return false;
		}

		TotalBytesSent += Payload.Num();

		return true;
	}

	return false;
}


/* FRunnable interface
 *****************************************************************************/

bool FTcpMessageTransportConnection::Init()
{
	return (Socket != nullptr);
}

uint32 FTcpMessageTransportConnection::Run()
{
	UE_LOGF(LogTcpMessaging, Log, "Started Connection to '%ls'", *RemoteEndpoint.ToString());

	while (bRun)
	{
		// Try sending the header if needed, and receiving messages and detect if they fail or if another connection error is reported.
		if ((!SendHeader() || !ReceiveMessages() || Socket->GetConnectionState() == SCS_ConnectionError) && bRun)
		{
			// Disconnected. Reconnect if requested.
			const float Delay = ConnectionRetryDelay;
			if (Delay > 0)
			{
				bool bReconnectPending = false;
				{
				    // Wait for any sending before we close the socket
				    FScopeLock SendLock(&SendCriticalSection);
    
				    Socket->Close();
				    ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->DestroySocket(Socket);
					Socket = nullptr;

					for (float TimeSpentRetrying = 0; bRun && Socket == nullptr && TimeSpentRetrying <= ConnectionRetryPeriod; TimeSpentRetrying += Delay)
					{
						UE_LOGF(LogTcpMessaging, Verbose, "Connection to '%ls' failed, retrying...", *RemoteEndpoint.ToString());
						FPlatformProcess::Sleep(ConnectionRetryDelay);

						Socket = FTcpSocketBuilder(TEXT("FTcpMessageTransport.RemoteConnection"))
							.WithSendBufferSize(TCP_MESSAGING_SEND_BUFFER_SIZE)
							.WithReceiveBufferSize(TCP_MESSAGING_RECEIVE_BUFFER_SIZE);

						if (Socket && Socket->Connect(RemoteEndpoint.ToInternetAddr().Get()))
						{
							bSentHeader = false;
							bReceivedHeader = false;
							ConnectionState = STATE_DisconnectReconnectPending;
							RemoteNodeId.Invalidate();
							bReconnectPending = true;
						}
						else
						{
							if (Socket)
							{
								ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->DestroySocket(Socket);
								Socket = nullptr;
								bRun = false;
							}
						}
					}
				}

				if (bReconnectPending)
				{
					ConnectionStateChangedDelegate.ExecuteIfBound();
				}
				else
				{
					UE_LOGF(LogTcpMessaging, Error, "Reconnection to '%ls' failed.", *RemoteEndpoint.ToString());
				}
			}
			else
			{
				bRun = false;
			}
		}

		FPlatformProcess::SleepNoStats(0.0001f);
	}

	{
		FScopeLock SendLock(&SendCriticalSection);
		ConnectionState = STATE_Disconnected;
	}
	ConnectionStateChangedDelegate.ExecuteIfBound();
	
	RemoteNodeId.Invalidate();
	ClosedTime = FDateTime::UtcNow();

	// Clear the delegate to remove a reference to this connection
	ConnectionStateChangedDelegate.Unbind();
	return 0;
}

void FTcpMessageTransportConnection::Stop()
{
	if (Socket)
	{
		Socket->Close();
	}
}

void FTcpMessageTransportConnection::Exit()
{
	// do nothing
}


/* FTcpMessageTransportConnection implementation
 *****************************************************************************/

void FTcpMessageTransportConnection::Close()
{
	// let the thread shutdown on its own
	if (Thread != nullptr)
	{
		bRun = false;
		Thread->WaitForCompletion();
		delete Thread;
		Thread = nullptr;
	}

	// if there a socket, close it so our peer will get a quick disconnect notification
	if (Socket)
	{
		Socket->Close();
	}
}

uint64 FTcpMessageTransportConnection::GetTotalBytesReceived() const
{
	return TotalBytesReceived;
}

uint64 FTcpMessageTransportConnection::GetTotalBytesSent() const
{
	return TotalBytesSent;
}

FText FTcpMessageTransportConnection::GetName() const
{
	return RemoteEndpoint.ToText();
}

FTimespan FTcpMessageTransportConnection::GetUptime() const
{
	if (ConnectionState == STATE_Connected)
	{
		return (FDateTime::UtcNow() - OpenedTime);
	}

	return (ClosedTime - OpenedTime);
}

FTcpMessageTransportConnection::EConnectionState FTcpMessageTransportConnection::GetConnectionState() const
{
	return ConnectionState;
}

FGuid FTcpMessageTransportConnection::GetRemoteNodeId() const
{
	return RemoteNodeId;
}

bool FTcpMessageTransportConnection::ReceiveMessages()
{
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	uint32 PendingDataSize = 0;

	auto GetReadableErrorCode = [SocketSubsystem]() -> FString
	{
		ESocketErrors LastError = SocketSubsystem->GetLastErrorCode();
		return FString::Printf(TEXT("%s (%d)"), SocketSubsystem->GetSocketError(LastError), (int32)LastError);
	};

	// check if the socket has closed
	{
		int32 BytesRead;
		uint8 Dummy;
		if (!Socket->Recv(&Dummy, 1, BytesRead, ESocketReceiveFlags::Peek))
		{
			UE_LOGF(LogTcpMessaging, Verbose, "Dummy read failed with code %ls", *GetReadableErrorCode());
			return false;
		}
	}

	// Block waiting for some data
	if (!Socket->Wait(ESocketWaitConditions::WaitForRead, FTimespan::FromSeconds(1.0)))
	{
		return (Socket->GetConnectionState() != SCS_ConnectionError);
	}

	if (!bReceivedHeader)
	{
		if (Socket->HasPendingData(PendingDataSize) && PendingDataSize >= sizeof(FTcpMessageHeader))
		{
			FArrayReader HeaderData = FArrayReader(true);
			HeaderData.SetNumUninitialized(sizeof(FTcpMessageHeader));
			int32 BytesRead = 0;
			if (!Socket->Recv(HeaderData.GetData(), sizeof(FTcpMessageHeader), BytesRead))
			{
				UE_LOGF(LogTcpMessaging, Verbose, "Header read failed with code %ls", *GetReadableErrorCode());
				return false;
			}

			check(BytesRead == sizeof(FTcpMessageHeader));
			TotalBytesReceived += BytesRead;

			FTcpMessageHeader MessageHeader;
			HeaderData << MessageHeader;

			if (!MessageHeader.IsValid())
			{
				UE_LOGF(LogTcpMessaging, Verbose, "Header read failed with invalid header");
				return false;
			}
			else
			{
				RemoteNodeId = MessageHeader.GetNodeId();
				RemoteProtocolVersion = MessageHeader.GetVersion();
				bReceivedHeader = true;
				OpenedTime = FDateTime::UtcNow();
				{
					FScopeLock SendLock(&SendCriticalSection);
					ConnectionState = STATE_Connected;
				}
				ConnectionStateChangedDelegate.ExecuteIfBound();
			}
		}
		else
		{
			// no header yet
			return true;
		}
	}

	// keep going until we have no data.
	for(;;)
	{
		int32 BytesRead = 0;
		// See if we're in the process of receiving a (large) message
		if (RecvMessageDataRemaining == 0)
		{
			// no partial message. Try to receive the size of a message
			if (!Socket->HasPendingData(PendingDataSize) || (PendingDataSize < sizeof(uint32)))
			{
				// no messages
				return true;
			}

			FArrayReader MessagesizeData = FArrayReader(true);
			MessagesizeData.SetNumUninitialized(sizeof(uint32));

			// read message size from the stream
			BytesRead = 0;
			if (!Socket->Recv(MessagesizeData.GetData(), sizeof(uint32), BytesRead))
			{
				UE_LOGF(LogTcpMessaging, Verbose, "In progress read failed with code %ls", *GetReadableErrorCode());
				return false;
			}

			check(BytesRead == sizeof(uint32));
			TotalBytesReceived += BytesRead;

			// Setup variables to receive the message
			MessagesizeData << RecvMessageDataRemaining;

			if (RecvMessageDataRemaining <= 0)
			{
				UE_LOGF(LogTcpMessaging, Error, "Read failed due to invalid Message Size: %d", RecvMessageDataRemaining);
				return false;
			}

			RecvMessageData = MakeShareable(new FArrayReader(true));
			RecvMessageData->SetNumUninitialized(RecvMessageDataRemaining);
		}

		BytesRead = 0;
		if (!Socket->Recv(RecvMessageData->GetData() + RecvMessageData->Num() - RecvMessageDataRemaining, RecvMessageDataRemaining, BytesRead))
		{
			UE_LOGF(LogTcpMessaging, Verbose, "Read failed with code %ls", *GetReadableErrorCode());
			return false;
		}

		if (BytesRead > 0)
		{
			check(BytesRead <= RecvMessageDataRemaining);
			TotalBytesReceived += BytesRead;
			RecvMessageDataRemaining -= BytesRead;
			if (RecvMessageDataRemaining == 0)
			{
				TSharedPtr<FTcpDeserializedMessage, ESPMode::ThreadSafe> DeserializedMessage = MakeShareable(new FTcpDeserializedMessage(nullptr));
				if (DeserializedMessage->Deserialize(RecvMessageData))
				{
					Inbox.Enqueue(DeserializedMessage);
				}
				RecvMessageData.Reset();
			}
		}
		else
		{
			// no data
			return true;
		}
	}
}

bool FTcpMessageTransportConnection::ChunkedBlockingSend(const TArray<uint8>& Payload)
{
	// send the payload
	if (!BlockingSend(Payload.GetData(), Payload.Num()))
	{
		ESocketErrors LastError = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLastErrorCode();
		UE_LOGF(LogTcpMessaging, Verbose, "Payload write failed with code %d", (int32)LastError);

		if (LastError == SE_ENOBUFS)
		{
			UE_LOGF(LogTcpMessaging, Verbose, "Could not send %d bytes in one blocking send. Trying really small chunks.", Payload.Num());

			const int32 Max64kb = 64*1024;
			const uint32 MaxRetries = 16;
			uint32 NumRetries = 0;

			FPlatformProcess::Sleep(1);
			const uint8* Buf = Payload.GetData();
			int32 BufSz = Payload.Num();
			while (BufSz > 0)
			{
				int SendMax = FMath::Min(BufSz, Max64kb);
				if (BlockingSend(Buf, SendMax))
				{
					Buf += SendMax;
					BufSz -= SendMax;
				}
				else if (NumRetries < MaxRetries)
				{
					FPlatformProcess::Sleep(1);
					NumRetries++;
					UE_LOGF(LogTcpMessaging, Verbose, "%d retries sending small chunks", NumRetries);
				}
				else
				{
					// The target could have disconnected in the middle of the transmission.
					return false;
				}
			}
		}
		else
		{
			return false;
		}
	}

	return true;
}

bool FTcpMessageTransportConnection::BlockingSend(const uint8* Data, int32 BytesToSend)
{
	int32 TotalBytes = BytesToSend;
	while (BytesToSend > 0)
	{
		while (!Socket->Wait(ESocketWaitConditions::WaitForWrite, FTimespan::FromSeconds(1.0)))
		{
			if (Socket->GetConnectionState() == SCS_ConnectionError)
			{
				return false;
			}
		}

		int32 BytesSent = 0;
		if (!Socket->Send(Data, BytesToSend, BytesSent))
		{
			return false;
		}
		BytesToSend -= BytesSent;
		Data += BytesSent;
	}
	return true;
}

bool FTcpMessageTransportConnection::SendHeader()
{
	if (bSentHeader)
	{
		return true;
	}

	FScopeLock SendLock(&SendCriticalSection);

	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);

	// See if we're writable
	if (!Socket->Wait(ESocketWaitConditions::WaitForWrite, FTimespan::FromSeconds(5)))
	{
		return true;
	}

	FArrayWriter HeaderData;
	FTcpMessageHeader MessageHeader(LocalNodeId);
	HeaderData << MessageHeader;

	if (!BlockingSend(HeaderData.GetData(), sizeof(FTcpMessageHeader)))
	{
		UE_LOGF(LogTcpMessaging, Verbose, "Header write failed with code %d", (int32)SocketSubsystem->GetLastErrorCode());
		return false;
	}

	bSentHeader = true;
	TotalBytesSent += sizeof(FTcpMessageHeader);

	return true;
}

