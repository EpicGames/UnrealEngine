// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/DateTime.h"
#include "Misc/Guid.h"
#include "Transport/TcpSerializedMessage.h"
#include "HAL/Runnable.h"
#include "Containers/Queue.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"

class FArrayReader;

class FTcpDeserializedMessage;

/** Delegate type for announcing a connection state change */
DECLARE_DELEGATE(FOnTcpMessageTransportConnectionStateChanged)

/**
 * Implements a TCP message tunnel connection.
 */
class FTcpMessageTransportConnection
	: public FRunnable
	, public TSharedFromThis<FTcpMessageTransportConnection>
 {
public:

	/** Connection state */
	enum EConnectionState
	{
		STATE_Connecting,					// connecting but don't yet have RemoteNodeId
		STATE_Connected,					// connected and RemoteNodeId is valid
		STATE_DisconnectReconnectPending,	// disconnected with reconnect pending. Previous RemoteNodeId is retained
		STATE_Disconnected					// disconnected. Previous RemoteNodeId is retained
	};

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InSocket The socket to use for this connection.
	 * @param InRemoteAddress The IP endpoint of the remote client.
	 * @param int32 InConnectionRetryDelay Delay before retrying connection, 0 disables
	 * @param int32 InConnectionRetryPeriod period during which retrying connection, 0 means to retry only once
	 */
	FTcpMessageTransportConnection(FSocket* InSocket, const FIPv4Endpoint& InRemoteEndpoint, int32 InConnectionRetryDelay, int32 InConnectionRetryPeriod = 0);

	/** Virtual destructor. */
	virtual ~FTcpMessageTransportConnection();


	/**
	 * Starts processing of the connection. Needs to be called immediately after construction
	 */
	void Start();

	/**
	 * Receives a message from the connection's inbox.
	 *
	 * @param OutMessage will hold the received message, if any.
	 * @return true if a message was returned, false otherwise.
	 * @see Send
	 */
	bool Receive(TSharedPtr<FTcpDeserializedMessage, ESPMode::ThreadSafe>& OutMessage, FGuid& OutSenderNodeId);

	/**
	 * Sends a serialized message through this connection.
	 *
	 * @param Message The message to send.
	 * @see Receive
	 */
	bool Send(FTcpSerializedMessagePtr Message);

	/** Closes this connection. */
	void Close();

	/**
	* Gets the total number of bytes received from this connection.
	*
	* @return Number of bytes.
	*/
	uint64 GetTotalBytesReceived() const;

	/**
	* Gets the total number of bytes received from this connection.
	*
	* @return Number of bytes.
	*/
	uint64 GetTotalBytesSent() const;

	/**
	* Gets the human readable name of the connection.
	*
	* @return Connection name.
	*/
	FText GetName() const;

	/**
	* Gets the amount of time that the connection has been established.
	*
	* @return Time span.
	*/
	FTimespan GetUptime() const;

	/**
	* Gets the current state of the connection.
	*
	* @return EConnectionState current state.
	*/
	EConnectionState GetConnectionState() const;

	/**
	* Gets the remote node's id
	*
	* @return FGuid the remote node's id, or and invalid FGuid if there is no current valid remote node id.
	*/
	FGuid GetRemoteNodeId() const;

	/**
	* Gets the IP address and port of the remote connection
	*
	* @return FIPv4Endpoint remote connection's IP address and port
	*/
	FIPv4Endpoint GetRemoteEndpoint() const { return RemoteEndpoint; }

	/**
	* Gets the delegate which will be called whenever the connection state changes
	*
	* @return FOnTcpMessageTransportConnectionStateChanged delegate
	*/
	FOnTcpMessageTransportConnectionStateChanged& OnTcpMessageTransportConnectionStateChanged()
	{
		return ConnectionStateChangedDelegate;
	}

 protected:

	 /**
	 * Receives all pending messages from the socket.
	 *
	 * @return true on success, false otherwise.
	 * @see SendMessages
	 */
	 bool ReceiveMessages();

	 /**
	 * Sends header to the socket.
	 *
	 * @return true on success, false otherwise.
	 */
	 bool SendHeader();

private:

	//~ FRunnable interface

	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override;

private:

	/** Split the send data into chunks until all data is sent. */
	bool ChunkedBlockingSend(const TArray<uint8>& Payload);

	/** Try to send data, but if all data is not sent in one go, block on send until data is sent or an error occurs */
	bool BlockingSend(const uint8* Data, int32 BytesToSend);

	/** Connection state changed delegate */
	FOnTcpMessageTransportConnectionStateChanged ConnectionStateChangedDelegate;

	/** Current connection state */
	EConnectionState ConnectionState;

	/** Holds the time at which the connection was closed. */
	FDateTime ClosedTime;

	/** Holds the collection of received Messages. */
	TQueue<TSharedPtr<FTcpDeserializedMessage, ESPMode::ThreadSafe>, EQueueMode::Mpsc> Inbox;

	/** Holds the time at which the connection was opened. */
	FDateTime OpenedTime;

	/** Holds the IP endpoint of the remote client. */
	FIPv4Endpoint RemoteEndpoint;

	/** Local Node Id */
	FGuid LocalNodeId;

	/** Remote Node Id */
	FGuid RemoteNodeId;

	/** Whether we've sent the initial header to the remote end */
	bool bSentHeader;

	/** Whether we've received the initial header from the remote end */
	bool bReceivedHeader;

	/** Peer's value for of ETcpMessagingVersion::LatestVersion */
	uint32 RemoteProtocolVersion;

	/** Holds the connection socket. */
	FSocket* Socket;

	/** Holds the thread object. */
	FRunnableThread* Thread;

	/** Holds the total number of bytes received from the connection. */
	uint64 TotalBytesReceived;

	/** Holds the total number of bytes sent to the connection. */
	uint64 TotalBytesSent;

	/** thread should continue running. */
	bool bRun;

	/** Delay before re-establishing connection if it drops, 0 disables */
	int32 ConnectionRetryDelay;

	/** Period during which re-establishing connection if it drops, 0 means it will retry only once */
	int32 ConnectionRetryPeriod;

	/** Message data we're currently in the process of receiving, if any */
	TSharedPtr<FArrayReader, ESPMode::ThreadSafe> RecvMessageData;

	/** The number of bytes of incoming message data we're still waiting on before we have a complete message */
	int32 RecvMessageDataRemaining;

	/** Critical section preventing multiple threads from sending simultaneously */
	FCriticalSection SendCriticalSection;
};

