// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transport/TcpSerializeMessageTask.h"
#include "IMessageContext.h"
#include "Transport/TcpMessageTransportConnection.h"
#include "Backends/JsonStructSerializerBackend.h"
#include "StructSerializer.h"
#include "TcpSerializedMessage.h"


/* FTcpSerializeMessageTask interface
 *****************************************************************************/

void FTcpSerializeMessageTask::DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	if (MessageContext->IsValid())
	{
		// Note that some complex values are serialized manually here, so that we can ensure
		// a consistent wire format, if their implementations change. This allows us to sanity
		// check the values during deserialization. @see FTcpDeserializeMessage::Deserialize()

		// serialize context
		FArchive& Archive = SerializedMessage.Get();
		{
			const FTopLevelAssetPath& MessageType = MessageContext->GetMessageTypePathName();
			Archive << const_cast<FTopLevelAssetPath&>(MessageType);

			const FMessageAddress& Sender = MessageContext->GetSender();
			Archive << const_cast<FMessageAddress&>(Sender);

			const TArray<FMessageAddress>& Recipients = MessageContext->GetRecipients();
			Archive << const_cast<TArray<FMessageAddress>&>(Recipients);

			EMessageScope Scope = MessageContext->GetScope();
			Archive << Scope;

			const FDateTime& TimeSent = MessageContext->GetTimeSent();
			Archive << const_cast<FDateTime&>(TimeSent);

			const FDateTime& Expiration = MessageContext->GetExpiration();
			Archive << const_cast<FDateTime&>(Expiration);

			int32 NumAnnotations = MessageContext->GetAnnotations().Num();
			Archive << NumAnnotations;

			for (const auto& AnnotationPair : MessageContext->GetAnnotations())
			{
				Archive << const_cast<FName&>(AnnotationPair.Key);
				Archive << const_cast<FString&>(AnnotationPair.Value);
			}
		}

		// serialize message body
		UScriptStruct* MessageTypeInfoPtr = MessageContext->GetMessageTypeInfo().Get();
		FJsonStructSerializerBackend Backend(Archive, EStructSerializerBackendFlags::None);
		FStructSerializer::Serialize(MessageContext->GetMessage(), *MessageTypeInfoPtr, Backend);

		// enqueue to recipients
		for (auto& Connection : RecipientConnections)
		{
			Connection->Send(SerializedMessage);
		}
	}
}


ENamedThreads::Type FTcpSerializeMessageTask::GetDesiredThread()
{
	return ENamedThreads::AnyThread;
}


TStatId FTcpSerializeMessageTask::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FTcpSerializeMessageTask, STATGROUP_TaskGraphTasks);
}


ESubsequentsMode::Type FTcpSerializeMessageTask::GetSubsequentsMode() 
{ 
	return ESubsequentsMode::FireAndForget; 
}
