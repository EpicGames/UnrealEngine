// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "Transport/TcpSerializedMessage.h"

class IMessageContext;
struct TStatId;

class FTcpMessageTransportConnection;

/**
 * Implements an asynchronous task for serializing a message.
 */
class FTcpSerializeMessageTask
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InMessageContext The context of the message to serialize.
	 * @param InSerializedMessage Will hold the serialized message data.
	 */
	FTcpSerializeMessageTask(TSharedRef<IMessageContext, ESPMode::ThreadSafe> InMessageContext, FTcpSerializedMessageRef InSerializedMessage, const TArray<TSharedPtr<FTcpMessageTransportConnection>>& InRecipientConnections)
		: MessageContext(InMessageContext)
		, SerializedMessage(InSerializedMessage)
		, RecipientConnections(InRecipientConnections)
	{ }

public:

	/**
	 * Performs the actual task.
	 *
	 * @param CurrentThread The thread that this task is executing on.
	 * @param MyCompletionGraphEvent The completion event.
	 */
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent);
	
	/**
	 * Returns the name of the thread that this task should run on.
	 *
	 * @return Thread name.
	 */
	ENamedThreads::Type GetDesiredThread();

	/**
	 * Gets the task's stats tracking identifier.
	 *
	 * @return Stats identifier.
	 */
	TStatId GetStatId() const;

	/**
	 * Gets the mode for tracking subsequent tasks.
	 *
	 * @return Always track subsequent tasks.
	 */
	static ESubsequentsMode::Type GetSubsequentsMode();

private:

	/** Holds the context of the message to serialize. */
	TSharedRef<IMessageContext, ESPMode::ThreadSafe> MessageContext;

	/** Holds a reference to the serialized message data. */
	FTcpSerializedMessageRef SerializedMessage;

	/** Connections we're going to enqueue the serialized message for */
	TArray<TSharedPtr<FTcpMessageTransportConnection>> RecipientConnections;
};
