// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Templates/SharedPointer.h"

/**
 * Holds serialized message data.
 */
class FTcpSerializedMessage
	: public FMemoryWriter
{
public:

	/** Default constructor. */
	FTcpSerializedMessage()
	: FMemoryWriter(DataArray, true)
	{}

public:

	/**
	 * Creates an archive reader to the data.
	 *
	 * The caller is responsible for deleting the returned object.
	 *
	 * @return An archive reader.
	 */
	FArchive* CreateReader()
	{
		return new FMemoryReader(DataArray, true);
	}

	/**
	 * Get the serialized message data.
	 *
	 * @return Byte array of message data.
	 * @see GetState
	 */
	const TArray<uint8>& GetDataArray()
	{
		return DataArray;
	}

private:

	/** Holds the serialized data. */
	TArray<uint8> DataArray;
};


/** Type definition for shared pointers to instances of FTcpSerializedMessage. */
typedef TSharedPtr<FTcpSerializedMessage, ESPMode::ThreadSafe> FTcpSerializedMessagePtr;

/** Type definition for shared references to instances of FTcpSerializedMessage. */
typedef TSharedRef<FTcpSerializedMessage, ESPMode::ThreadSafe> FTcpSerializedMessageRef;
