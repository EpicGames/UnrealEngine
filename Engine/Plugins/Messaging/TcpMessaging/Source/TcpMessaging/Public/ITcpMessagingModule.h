// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

class FString;


/**
 * IConnectionBasedMessagingModule interface allows adding/removing of connections.
 */
class ITcpMessagingModule
	: public IModuleInterface
{
public:

	virtual void AddOutgoingConnection(const FString& Endpoint) = 0;
	virtual void RemoveOutgoingConnection(const FString& Endpoint) = 0;
};
