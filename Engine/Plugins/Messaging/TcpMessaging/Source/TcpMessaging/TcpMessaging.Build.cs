// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class TcpMessaging : ModuleRules
	{
		public TcpMessaging(ReadOnlyTargetRules Target) : base(Target)
		{
			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"Messaging",
				});

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"CoreUObject",
					"Networking",
					"Serialization",
					"Sockets",
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Messaging",
					"MessagingCommon",
				});

			PrivateIncludePaths.AddRange(
				new string[] {
					"TcpMessaging/Private/Settings",
					"TcpMessaging/Private/Transport",
				});

			if (Target.bCompileAgainstEditor)
			{
				DynamicallyLoadedModuleNames.AddRange(
					new string[] {
						"Settings",
					});

				PrivateIncludePathModuleNames.AddRange(
					new string[] {
						"Settings",
					});
			}
		}
	}
}
