{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "TCP Messaging",
	"Description" : "Adds a TCP connection based transport layer to the messaging sub-system for sending and receiving messages between networked computers and devices.",
	"Category" : "Messaging",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"SupportedPrograms": [ "UnrealFrontend" , "UnrealInsights", "UnrealPak" ],
	"Modules" :
	[
		{
			"Name" : "TcpMessaging",
			"Type" : "Runtime",
			"LoadingPhase" : "PreDefault",
			"PlatformDenyList" :
			[
			],
			"ProgramAllowList": [ "UnrealFrontend", "UnrealInsights", "UnrealPak" ]
	    }
	]
}
